// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ConcertInsightsCore : ModuleRules
{
    public ConcertInsightsCore(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "Concert",
	            "ConcertTransport",
                "CoreUObject",
                "Slate",
                "SlateCore",
                "TraceLog",
            }
        );

        ShortName = "CrtInC";
    }
}