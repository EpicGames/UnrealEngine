// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertInsightsSynchronizedTrace.h"

#define LOCTEXT_NAMESPACE "FConcertInsightsSynchronizedTraceModule"

namespace UE::ConcertInsightsSyncTrace
{
	void FConcertInsightsSyncTraceModule::StartupModule()
	{
    
	}

	void FConcertInsightsSyncTraceModule::ShutdownModule()
	{
    
	}
}


#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(UE::ConcertInsightsSyncTrace::FConcertInsightsSyncTraceModule, ConcertInsightsCore)