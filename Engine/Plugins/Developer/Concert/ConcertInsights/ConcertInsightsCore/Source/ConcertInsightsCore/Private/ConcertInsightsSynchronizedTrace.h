// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertInsightsSyncTraceModule.h"

namespace UE::ConcertInsightsSyncTrace
{
	class FConcertInsightsSyncTraceModule : public IConcertInsightsSyncTraceModule
	{
	public:

		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface
	};
}

