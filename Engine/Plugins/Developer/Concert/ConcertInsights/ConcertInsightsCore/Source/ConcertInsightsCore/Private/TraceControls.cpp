// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceControls.h"

#include "ConcertLogGlobal.h"
#include "Concert/Public/IConcertSession.h"
#include "Trace/ConcertProtocolTrace.h"

#include "ProfilingDebugging/TraceAuxiliary.h"

#define LOCTEXT_NAMESPACE "FTraceControls"

namespace UE::ConcertInsightsCore
{
	FTraceControls::FTraceControls()
	{
		FTraceAuxiliary::OnTraceStarted.AddRaw(this, &FTraceControls::OnTraceStarted);
		FTraceAuxiliary::OnTraceStopped.AddRaw(this, &FTraceControls::OnTraceStopped);
	}

	FTraceControls::~FTraceControls()
	{
		FTraceAuxiliary::OnTraceStopped.RemoveAll(this);
	}

	bool FTraceControls::IsTracing() const
	{
		return FTraceAuxiliary::IsConnected();
	}

#define SET_REASON(FailReasonVar, ReasonLocText) if (FailReason) { *FailReasonVar = ReasonLocText; }
	
	bool FTraceControls::StartSynchronizedTrace(TSharedRef<IConcertSession> Session, const FStartTraceArgs& Args, FText* FailReason)
	{
		if (IsTracing())
		{
			SET_REASON(FailReason, LOCTEXT("Reason.AlreadyRunning", "A trace is already in progress"));
			return false;
		}

		if (!LocallyStartSynchronizedTrace({ Session }, Args))
		{
			SET_REASON(FailReason, LOCTEXT("Reason.FailedToStart", "Failed to start local trace. See log."));
			return false;
		}
		
		FConcertTrace_StartSyncTrace_Request Request;
		Request.TraceArgs = static_cast<FConcertTrace_StartTraceArgs>(Args);
		auto SendRequest = [this, &Session, &Request](const FGuid& Endpoint)
		{
			if (CanSendRequestsToEndpoint(Endpoint, *Session))
			{
				UE_LOGF(LogConcert, Verbose, "Sending synchronized trace request to client %ls", *Endpoint.ToString());
				
				Session->SendCustomRequest<FConcertTrace_StartSyncTrace_Request, FConcertTrace_StartSyncTrace_Response>(Request, Endpoint)
					.Next([Endpoint](FConcertTrace_StartSyncTrace_Response&& Response)
					{
						UE_CLOGF(Response.ErrorCode == EConcertTraceErrorCode::Timeout, LogConcert, Warning, "Client %ls timed out answering synchronized trace request", *Endpoint.ToString());
						UE_CLOGF(Response.ErrorCode == EConcertTraceErrorCode::Joined, LogConcert, Log, "Client %ls accepted synchronized trace request", *Endpoint.ToString());
						UE_CLOGF(Response.ErrorCode == EConcertTraceErrorCode::Rejected, LogConcert, Error, "Client %ls rejected synchronized trace request", *Endpoint.ToString());
					});
			}
		};
		
		SendRequest(Session->GetSessionInfo().ServerEndpointId);
		for (const FGuid& ClientEndpoint : Session->GetSessionClientEndpointIds())
		{
			SendRequest(ClientEndpoint);
		}

		return true;
	}

	void FTraceControls::StopSynchronizedTrace()
	{
		if (!InProgressSynchronizedTrace)
		{
			return;
		}
		UE_LOGF(LogConcert, Log, "Stopping synchronized trace...");
		
		const TSharedPtr<IConcertSession> Session = InProgressSynchronizedTrace->TraceInitiator.Pin();
		// Reset before calling FTraceAuxiliary::Stop() so OnTraceStopped does not cause a recursive call
		InProgressSynchronizedTrace.Reset();
		
		if (Session)
		{
			FTraceAuxiliary::Stop();
			
			auto SendRequest = [this, &Session](const FGuid& Endpoint)
			{
				Session->SendCustomEvent(FConcertTrace_StopSyncTrace{}, { Endpoint } , EConcertMessageFlags::ReliableOrdered);
			};

			const FGuid ServerEndpoint = Session->GetSessionInfo().ServerEndpointId;
			if (CanSendRequestsToEndpoint(ServerEndpoint, *Session))
			{
				SendRequest(ServerEndpoint);
			}
			for (const FGuid& ClientEndpointId : Session->GetSessionClientEndpointIds())
			{
				SendRequest(ClientEndpointId);
			}
		}
		
		OnSynchronizedTraceStoppedDelegate.Broadcast();
	}

	void FTraceControls::RegisterTraceRequestsHandler(const TSharedRef<IConcertSession>& Session)
	{
		if (!ensure(!RegisteredSessions.Contains(Session)))
		{
			return;
		}
		RegisteredSessions.Add(Session);
		
		Session->RegisterCustomRequestHandler<FConcertTrace_StartSyncTrace_Request, FConcertTrace_StartSyncTrace_Response>(
			[this, WeakSession = TWeakPtr<IConcertSession>(Session)](const FConcertSessionContext& Context, const FConcertTrace_StartSyncTrace_Request& Request, FConcertTrace_StartSyncTrace_Response& Response)
			{
				const TSharedPtr<IConcertSession> SessionPin = WeakSession.Pin();
				check(SessionPin);
				return HandleTraceStartRequest(Context, Request, Response, SessionPin.ToSharedRef());
			});
		Session->RegisterCustomEventHandler<FConcertTrace_StopSyncTrace>(this, &FTraceControls::HandleTraceStopRequest);
	}

	void FTraceControls::OnLeaveSession(IConcertSession& Session)
	{
		if (IsInSynchronizedTrace())
		{
			StopLocalConcertTrace();
		}
	}

	EConcertSessionResponseCode FTraceControls::HandleTraceStartRequest(
		const FConcertSessionContext& Context,
		const FConcertTrace_StartSyncTrace_Request& Request,
		FConcertTrace_StartSyncTrace_Response& Response,
		const TSharedRef<IConcertSession>& Session
		)
	{
		if (!FTraceAuxiliary::IsConnected())
		{
			Response.ErrorCode = EConcertTraceErrorCode::Joined;

			LocallyStartSynchronizedTrace({ Session }, FStartTraceArgs{{ Request.TraceArgs }});
			OnSynchronizedTraceAccepted(Context, Request, Session);
		}
		else
		{
			Response.ErrorCode = EConcertTraceErrorCode::Rejected;
		}
		
		return EConcertSessionResponseCode::Success;
	}

	void FTraceControls::OnTraceStopped(FTraceAuxiliary::EConnectionType, const FString&)
	{
		if (InProgressSynchronizedTrace.IsSet())
		{
			StopSynchronizedTrace();
		}
	}

	bool FTraceControls::StartLocalConcertTrace(const FStartTraceArgs& Args)
	{
		Trace::ToggleChannel(TEXT("ConcertChannel"), true);
		
		if (!FTraceAuxiliary::IsConnected())
		{
			return FTraceAuxiliary::Start(
				ConcertInsightsSync::ConvertTraceTargetType(Args.ConnectionType),
				*Args.Target,
				*Args.Channels,
				Args.Options,
				Args.LogCategory
				);
		}

		// Trace already running, so successful.
		SendInitEventIfNeeded();
		return true;
	}

	void FTraceControls::StopLocalConcertTrace()
	{
		FTraceAuxiliary::Stop();
	}

	void FTraceControls::SendInitEventIfNeeded()
	{
		if (!FTraceAuxiliary::IsConnected())
		{
			return;
		}
		
		const TOptional<FInitArgs> InitArgs = GetInitEventArgs();
		if (!InitArgs)
		{
			return;
		}
		
		CONCERT_TRACE_INIT(InitArgs->EndpointId, InitArgs->DisplayString, InitArgs->bIsServer);
	}

	bool FTraceControls::LocallyStartSynchronizedTrace(FSynchronizedSessionData Data, const FStartTraceArgs& Args)
	{
		UE_LOGF(LogConcert, Log, "Starting synchronized trace.");
		
		check(!InProgressSynchronizedTrace.IsSet());
		InProgressSynchronizedTrace = MoveTemp(Data);

		if (StartLocalConcertTrace(Args))
		{
			OnSynchronizedTraceStartedDelegate.Broadcast();
			return true;
		}
		return false;
	}
}

#undef LOCTEXT_NAMESPACE