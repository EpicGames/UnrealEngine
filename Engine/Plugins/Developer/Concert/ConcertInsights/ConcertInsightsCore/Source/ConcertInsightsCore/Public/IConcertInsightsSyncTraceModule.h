// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

namespace UE::ConcertInsightsSyncTrace
{
	/**
	 * This module provides the base code for starting synchronized tracing across multiple machines.
	 * 
	 * Synchronized tracing means that certain machines in the session are requested to start tracing at the same time.
	 * Each machine will generate a separate .utrace file. When any of these files are analyzed in Unreal Insights, ConcertInsights collects the
	 * other relevant files and aggregates them in the UI.
	 * 
	 * This module houses the shared code needed to synchronize events across multiple machines, for example the Concert events that are sent to the other machines.
	 * ConcertInsightsServer and ConcertInsightsEditor depend on this module's exposed events.
	 */
	class CONCERTINSIGHTSCORE_API IConcertInsightsSyncTraceModule : public IModuleInterface
	{
	public:

		/**
		 * Singleton-like access to this module's interface.  This is just for convenience!
		 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
		 *
		 * @return Returns singleton instance, loading the module on demand if needed
		 */
		static inline IConcertInsightsSyncTraceModule& Get()
		{
			static const FName ModuleName = "ConcertInsightsCore";
			return FModuleManager::LoadModuleChecked<IConcertInsightsSyncTraceModule>(ModuleName);
		}

		/**
		 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
		 *
		 * @return True if the module is loaded and ready to use
		 */
		static inline bool IsAvailable()
		{
			static const FName ModuleName = "ConcertInsightsCore";
			return FModuleManager::Get().IsModuleLoaded(ModuleName);
		}
	};
}

