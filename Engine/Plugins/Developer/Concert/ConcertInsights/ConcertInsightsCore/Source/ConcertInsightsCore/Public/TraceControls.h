// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceMessages.h"
#include "Concert/Public/ConcertMessages.h"
#include "Concert/Public/IConcertSessionHandler.h"
#include "Templates/UnrealTemplate.h"

class IConcertSession;

namespace UE::ConcertInsightsCore
{
	struct FStartTraceArgs : FConcertTrace_StartTraceArgs
	{
		FTraceAuxiliary::FOptions* Options = nullptr;
		const FTraceAuxiliary::FLogCategoryAlias& LogCategory = LogCore;
	};

	struct FInitArgs
	{
		TOptional<FGuid> EndpointId;
		FString DisplayString;
		bool bIsServer;
	};

	/**
	 * Manages synchronized traces.
	 * 
	 * This listens for remote requests for starting traces.
	 * Intended to be subclassed.
	 */
	class CONCERTINSIGHTSCORE_API FTraceControls : public FNoncopyable
	{
	public:

		template<typename TControls, typename... TArgs>
		static TUniquePtr<TControls> Make(TArgs&&... Arg)
		{
			TUniquePtr<TControls> Result(new TControls(Forward<TArgs>(Arg)...));
			
			// Ensure that init event is traced if the engine was started with tracing enabled.
			// This calls virtual functions so must call after constructor.
			Result->SendInitEventIfNeeded();
			
			return Result;
		}
		
		virtual ~FTraceControls();

		/** Whether a trace is currently occuring */
		bool IsTracing() const;

		/** Starts a synchronized trace across multiple endpoints. */
		bool StartSynchronizedTrace(TSharedRef<IConcertSession> Session, const FStartTraceArgs& Args, FText* FailReason = nullptr);
		bool StartSynchronizedTrace(TSharedRef<IConcertSession> Session, FText* FailReason = nullptr) { return StartSynchronizedTrace(Session, GetDefaultSynchronizedTraceArgs(), FailReason); }

		/** Stops a synchronized trace if one is ongoing. */
		void StopSynchronizedTrace();

		/** Gets the default trace arguments to use */
		virtual FStartTraceArgs GetDefaultSynchronizedTraceArgs() const { return {}; }

		bool IsInSynchronizedTrace() const { return InProgressSynchronizedTrace.IsSet(); }

		DECLARE_EVENT(FTraceControls, FOnSynchronizedTraceStarted);
		/** Broadcasts when a synchronized trace is started for any reason, such as the local machine starting it or a remote request being accepted. */
		FOnSynchronizedTraceStarted& OnSynchronizedTraceStarted() { return OnSynchronizedTraceStartedDelegate; };
		
		DECLARE_EVENT(FTraceControls, FOnSynchronizedTraceStopped);
		/** Broadcasts when a synchronized trace is stopped for any reason, such as the local machine stopping it or being told so by a remote request. */
		FOnSynchronizedTraceStopped& OnSynchronizedTraceStopped() { return OnSynchronizedTraceStoppedDelegate; };
		
	protected:
		
		FTraceControls();
		
		/** Registers handlers for trace requests. */
		void RegisterTraceRequestsHandler(const TSharedRef<IConcertSession>& Session);

		/** Called by subclasses to notify that the local application has left the session. */
		void OnLeaveSession(IConcertSession& Session);

		/** @return Allows subclass to decide whether a trace start request should be sent to this endpoint. */
		virtual bool CanSendRequestsToEndpoint(const FGuid& EndpointId, const IConcertSession& Session) const { return true; }
		/** Called when an incoming synchronized trace was accepted. */
		virtual void OnSynchronizedTraceAccepted(const FConcertSessionContext& Context, const FConcertTrace_StartSyncTrace_Request& Request, const TSharedRef<IConcertSession>& Session) {}
		
		/** Checks whether joining a trace is ok and if so, return the data to put into the ConcertInsights init event. */
		virtual FInitArgs GetInitEventArgs() const = 0;

	private:

		struct FSynchronizedSessionData
		{
			/** The session that the synchronized trace was started on. */
			TWeakPtr<IConcertSession> TraceInitiator;
		};
		
		/** Sessions these controls are listening for requests on. */
		TSet<TWeakPtr<IConcertSession>> RegisteredSessions;
		
		/** Data for the currently running synchronized trace. Unset if not running. */
		TOptional<FSynchronizedSessionData> InProgressSynchronizedTrace;

		FOnSynchronizedTraceStarted OnSynchronizedTraceStartedDelegate;
		FOnSynchronizedTraceStopped OnSynchronizedTraceStoppedDelegate;

		/** Handles networked requests from other endpoints to start tracing. */
		EConcertSessionResponseCode HandleTraceStartRequest(const FConcertSessionContext& Context, const FConcertTrace_StartSyncTrace_Request& Request, FConcertTrace_StartSyncTrace_Response& Response, const TSharedRef<IConcertSession>& Session);
		/** Handles networked requests from other endpoints to stop tracing. */
		void HandleTraceStopRequest(const FConcertSessionContext&, const FConcertTrace_StopSyncTrace&) { StopLocalConcertTrace(); }
		
		/** Local callback for when tracing begins */
		void OnTraceStarted(FTraceAuxiliary::EConnectionType, const FString&) { SendInitEventIfNeeded(); }
		void OnTraceStopped(FTraceAuxiliary::EConnectionType, const FString&);
		
		/** Starts tracing (if not already), enables the Concert trace channel, and ensures everything for ConcertInsights is set up (init event is sent). */
		bool StartLocalConcertTrace(const FStartTraceArgs& Args);
		/** Stops tracing locally */
		void StopLocalConcertTrace();

		/** Sends the init event if we're tracing, the Concert channel is enabled, and the init event has not yet been sent during this session. */
		void SendInitEventIfNeeded();
		
		bool LocallyStartSynchronizedTrace(FSynchronizedSessionData Data, const FStartTraceArgs& Args);
	};
}



