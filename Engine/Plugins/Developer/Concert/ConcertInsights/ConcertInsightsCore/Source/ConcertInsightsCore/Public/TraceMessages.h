// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/TraceAuxiliary.h"
#include "TraceMessages.generated.h"

/** See /ref FTraceAuxiliary::EConnectionType for more info. */
UENUM()
enum class EConcertTraceTargetType : uint8
{
	/** Connect to a trace server. Target is IP address or hostname. */
	Network,
	/**
	 * Write to a file. Target string is filename. Absolute or relative current working directory.
	 * If target is null the current date and time is used.
	 */
	File,
};

namespace UE::ConcertInsightsSync
{
	FORCEINLINE FTraceAuxiliary::EConnectionType ConvertTraceTargetType(EConcertTraceTargetType ConnectionType)
	{
		return static_cast<FTraceAuxiliary::EConnectionType>(ConnectionType);
	}
}


/** Arguments to pass to FTraceAuxiliary::Start */
USTRUCT()
struct FConcertTrace_StartTraceArgs
{
	GENERATED_BODY()

	/** How target is supposed to be interpreted. */
	UPROPERTY()
	EConcertTraceTargetType ConnectionType = EConcertTraceTargetType::Network;

	/** String to use for connection. See /ref EConcertTraceTargetType for details */
	UPROPERTY()
	FString Target = TEXT("localhost");

	/** Channels to use. Comma separated. */
	UPROPERTY()
	FString Channels = TEXT("default,Concert");
};

/**
 * A request for the receive to:
 * - start tracing,
 * - ensure the .utrace contains a CONCERT_TRACE_INIT event
 */
USTRUCT()
struct FConcertTrace_StartSyncTrace_Request
{
	GENERATED_BODY()

	/** Info about how the trace is supposed to be started. */
	UPROPERTY()
	FConcertTrace_StartTraceArgs TraceArgs;
};

UENUM()
enum class EConcertTraceErrorCode : uint8
{
	Timeout,
	Joined,
	Rejected
};

USTRUCT()
struct FConcertTrace_StartSyncTrace_Response
{
	GENERATED_BODY()

	UPROPERTY()
	EConcertTraceErrorCode ErrorCode = EConcertTraceErrorCode::Timeout;
};

USTRUCT()
struct FConcertTrace_StopSyncTrace
{
	GENERATED_BODY()
};